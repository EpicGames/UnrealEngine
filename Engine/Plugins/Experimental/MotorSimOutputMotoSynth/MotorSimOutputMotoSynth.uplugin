{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1",
	"FriendlyName" : "MotorSimOutputMotoSynth",
	"Description" : "A MotorSim Output component using MotoSynth.",
	"Category" : "Audio",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"CanContainContent" : true,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Modules" : 
	[
		{
			"Name" : "MotorSimOutputMotoSynth",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins" :
		[
			{
				"Name" : "AudioMotorSim",
				"Enabled": true
			},
			{
				"Name" : "MotoSynth",
				"Enabled": true
			}
		]
}