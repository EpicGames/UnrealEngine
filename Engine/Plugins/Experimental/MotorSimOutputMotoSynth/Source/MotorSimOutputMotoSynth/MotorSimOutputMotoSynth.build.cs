// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MotorSimOutputMotoSynth : ModuleRules
	{
		public MotorSimOutputMotoSynth(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AudioExtensions",
					"MotoSynth",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AudioMixer",
					"AudioMotorSim",
				}
			);
		}
	}
}
