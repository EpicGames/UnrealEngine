// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotorSimOutputMotoSynth.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotorSimOutputMotoSynth)

void UMotorSimOutputMotoSynth::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	float MinRpm, MaxRpm;
	GetRPMRange(MinRpm, MaxRpm);

	if (!FMath::IsNearlyEqual(MinRpm, MaxRpm, KINDA_SMALL_NUMBER))
	{
		const float RpmLog = Audio::GetLogFrequencyClamped(RuntimeInfo.Rpm, { 0.0f, 1.0f }, { MinRpm, MaxRpm });

		SetRPM(RpmLog, Input.DeltaTime);
	}
}

void UMotorSimOutputMotoSynth::StartOutput()
{
	Start();
}
void UMotorSimOutputMotoSynth::StopOutput()
{
	Stop();
}

