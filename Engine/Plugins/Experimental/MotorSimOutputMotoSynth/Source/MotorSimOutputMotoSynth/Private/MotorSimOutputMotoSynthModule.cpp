// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotorSimOutputMotoSynthModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FMotorSimOutputMotoSynthModule, MotorSimOutputMotoSynth)

void FMotorSimOutputMotoSynthModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));
}

void FMotorSimOutputMotoSynthModule::ShutdownModule()
{
}


