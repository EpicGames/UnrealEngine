// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioMotorSimOutput.h"
#include "SynthComponents/SynthComponentMoto.h"
#include "MotorSimOutputMotoSynth.generated.h"

#define UE_API MOTORSIMOUTPUTMOTOSYNTH_API

UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class	UMotorSimOutputMotoSynth : public USynthComponentMoto, public IAudioMotorSimOutput
{
	GENERATED_BODY()

	virtual ~UMotorSimOutputMotoSynth() override = default;
	
public:
	UE_API virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;

	UE_API virtual void StartOutput() override;
	UE_API virtual void StopOutput() override;
};

#undef UE_API
